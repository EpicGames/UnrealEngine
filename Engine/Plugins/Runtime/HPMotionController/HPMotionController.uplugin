{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "HP Motion Controller",
	"Description": "Controller mappings for the HP Reverb G2 motion controller in OpenXR and SteamVR",
	"Category": "Virtual Reality",
	"CreatedBy": "Microsoft Corporation",
	"CreatedByURL": "https://www.microsoft.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": false,
	"IsBetaVersion": false,
	"Installed": false,
	"SupportedTargetPlatforms": [
		"Win64"
	],
	"Modules": [
		{
			"Name": "HPMotionController",
			"Type": "Runtime",
			"LoadingPhase": "PostConfigInit",
			"PlatformAllowList": [
				"Win64"
			]
		}
	],
	"Plugins": [
		{
			"Name": "OpenXR",
			"Enabled": true,
			"PlatformAllowList": [
				"Win64"
			]
		}
	]
}
