// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
    public class HPMotionController : ModuleRules
    {
        public HPMotionController(ReadOnlyTargetRules Target) 
				: base(Target)
        {
            PrivateDependencyModuleNames.AddRange(
                new string[]
                {
                    "Core",
                    "CoreUObject",
                    "Engine",
                    "InputCore",
					"OpenXRHMD",
				}
				);

            AddEngineThirdPartyPrivateStaticDependencies(Target, "OpenXR");
        }
    }
}
