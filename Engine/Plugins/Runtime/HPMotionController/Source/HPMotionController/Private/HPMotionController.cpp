// Copyright Epic Games, Inc. All Rights Reserved.

#include "HPMotionController.h"
#include "InputCoreTypes.h"
#include "OpenXRCore.h"
#include "Modules/ModuleManager.h"
#include "OpenXRAssetDirectory.h"

#define LOCTEXT_NAMESPACE "HPMotionControllerModule"

// Left
const FKey HPMixedRealityController_Left_X_Click("HPMixedRealityController_Left_X_Click");
const FKey HPMixedRealityController_Left_Y_Click("HPMixedRealityController_Left_Y_Click");
const FKey HPMixedRealityController_Left_Menu_Click("HPMixedRealityController_Left_Menu_Click");
const FKey HPMixedRealityController_Left_Grip("HPMixedRealityController_Left_Grip_Click");
const FKey HPMixedRealityController_Left_Grip_Axis("HPMixedRealityController_Left_Grip_Axis");
const FKey HPMixedRealityController_Left_Trigger("HPMixedRealityController_Left_Trigger_Click");
const FKey HPMixedRealityController_Left_Trigger_Axis("HPMixedRealityController_Left_Trigger_Axis");
const FKey HPMixedRealityController_Left_Thumbstick_X("HPMixedRealityController_Left_Thumbstick_X");
const FKey HPMixedRealityController_Left_Thumbstick_Y("HPMixedRealityController_Left_Thumbstick_Y");
const FKey HPMixedRealityController_Left_Thumbstick("HPMixedRealityController_Left_Thumbstick_Click");

//Right
const FKey HPMixedRealityController_Right_A_Click("HPMixedRealityController_Right_A_Click");
const FKey HPMixedRealityController_Right_B_Click("HPMixedRealityController_Right_B_Click");
const FKey HPMixedRealityController_Right_Menu_Click("HPMixedRealityController_Right_Menu_Click");
const FKey HPMixedRealityController_Right_Grip("HPMixedRealityController_Right_Grip_Click");
const FKey HPMixedRealityController_Right_Grip_Axis("HPMixedRealityController_Right_Grip_Axis");
const FKey HPMixedRealityController_Right_Trigger("HPMixedRealityController_Right_Trigger_Click");
const FKey HPMixedRealityController_Right_Trigger_Axis("HPMixedRealityController_Right_Trigger_Axis");
const FKey HPMixedRealityController_Right_Thumbstick_X("HPMixedRealityController_Right_Thumbstick_X");
const FKey HPMixedRealityController_Right_Thumbstick_Y("HPMixedRealityController_Right_Thumbstick_Y");
const FKey HPMixedRealityController_Right_Thumbstick("HPMixedRealityController_Right_Thumbstick_Click");


void FHPMotionControllerModule::StartupModule()
{
	RegisterOpenXRExtensionModularFeature();

	EKeys::AddMenuCategoryDisplayInfo("HPMixedRealityController",
		LOCTEXT("HPMixedRealityControllerSubCategory", "HP Mixed Reality Controller"), TEXT("GraphEditor.PadEvent_16x"));

	// Left
	EKeys::AddKey(FKeyDetails(HPMixedRealityController_Left_X_Click,
		LOCTEXT("HPMixedRealityController_Left_X_Click", "HP Mixed Reality (L) X Press"),
		FKeyDetails::GamepadKey | FKeyDetails::NotBlueprintBindableKey, "HPMixedRealityController"));
	EKeys::AddKey(FKeyDetails(HPMixedRealityController_Left_Y_Click,
		LOCTEXT("HPMixedRealityController_Left_Y_Click", "HP Mixed Reality (L) Y Press"),
		FKeyDetails::GamepadKey | FKeyDetails::NotBlueprintBindableKey, "HPMixedRealityController"));
	EKeys::AddKey(FKeyDetails(HPMixedRealityController_Left_Menu_Click,
		LOCTEXT("HPMixedRealityController_Left_Menu_Click", "HP Mixed Reality (L) Menu"),
		FKeyDetails::GamepadKey | FKeyDetails::NotBlueprintBindableKey, "HPMixedRealityController"));
	EKeys::AddKey(FKeyDetails(HPMixedRealityController_Left_Grip,
		LOCTEXT("HPMixedRealityController_Left_Grip_Click", "HP Mixed Reality (L) Grip"),
		FKeyDetails::GamepadKey | FKeyDetails::NotBlueprintBindableKey, "HPMixedRealityController"));
	EKeys::AddKey(FKeyDetails(HPMixedRealityController_Left_Grip_Axis,
		LOCTEXT("HPMixedRealityController_Left_Grip_Axis", "HP Mixed Reality (L) Grip Axis"),
		FKeyDetails::GamepadKey | FKeyDetails::Axis1D | FKeyDetails::NotBlueprintBindableKey, "HPMixedRealityController"));
	EKeys::AddKey(FKeyDetails(HPMixedRealityController_Left_Trigger,
		LOCTEXT("HPMixedRealityController_Left_Trigger_Click", "HP Mixed Reality (L) Trigger"),
		FKeyDetails::GamepadKey | FKeyDetails::NotBlueprintBindableKey, "HPMixedRealityController"));
	EKeys::AddKey(FKeyDetails(HPMixedRealityController_Left_Trigger_Axis,
		LOCTEXT("HPMixedRealityController_Left_Trigger_Axis", "HP Mixed Reality (L) Trigger Axis"),
		FKeyDetails::GamepadKey | FKeyDetails::Axis1D | FKeyDetails::NotBlueprintBindableKey, "HPMixedRealityController"));
	EKeys::AddKey(FKeyDetails(HPMixedRealityController_Left_Thumbstick_X,
		LOCTEXT("HPMixedRealityController_Left_Thumbstick_X", "HP Mixed Reality (L) Thumbstick X"),
		FKeyDetails::GamepadKey | FKeyDetails::Axis1D | FKeyDetails::NotBlueprintBindableKey, "HPMixedRealityController"));
	EKeys::AddKey(FKeyDetails(HPMixedRealityController_Left_Thumbstick_Y,
		LOCTEXT("HPMixedRealityController_Left_Thumbstick_Y", "HP Mixed Reality (L) Thumbstick Y"),
		FKeyDetails::GamepadKey | FKeyDetails::Axis1D | FKeyDetails::NotBlueprintBindableKey, "HPMixedRealityController"));
	EKeys::AddKey(FKeyDetails(HPMixedRealityController_Left_Thumbstick,
		LOCTEXT("HPMixedRealityController_Left_Thumbstick_Click", "HP Mixed Reality (L) Thumbstick"),
		FKeyDetails::GamepadKey | FKeyDetails::NotBlueprintBindableKey, "HPMixedRealityController"));

	// Right
	EKeys::AddKey(FKeyDetails(HPMixedRealityController_Right_A_Click,
		LOCTEXT("HPMixedRealityController_Right_A_Click", "HP Mixed Reality (R) A Press"),
		FKeyDetails::GamepadKey | FKeyDetails::NotBlueprintBindableKey, "HPMixedRealityController"));
	EKeys::AddKey(FKeyDetails(HPMixedRealityController_Right_B_Click,
		LOCTEXT("HPMixedRealityController_Right_B_Click", "HP Mixed Reality (R) B Press"),
		FKeyDetails::GamepadKey | FKeyDetails::NotBlueprintBindableKey, "HPMixedRealityController"));
	EKeys::AddKey(FKeyDetails(HPMixedRealityController_Right_Menu_Click,
		LOCTEXT("HPMixedRealityController_Right_Menu_Click", "HP Mixed Reality (R) Menu"),
		FKeyDetails::GamepadKey | FKeyDetails::NotBlueprintBindableKey, "HPMixedRealityController"));
	EKeys::AddKey(FKeyDetails(HPMixedRealityController_Right_Grip,
		LOCTEXT("HPMixedRealityController_Right_Grip_Click", "HP Mixed Reality (R) Grip"),
		FKeyDetails::GamepadKey | FKeyDetails::NotBlueprintBindableKey, "HPMixedRealityController"));
	EKeys::AddKey(FKeyDetails(HPMixedRealityController_Right_Grip_Axis,
		LOCTEXT("HPMixedRealityController_Right_Grip_Axis", "HP Mixed Reality (R) Grip Axis"),
		FKeyDetails::GamepadKey | FKeyDetails::Axis1D | FKeyDetails::NotBlueprintBindableKey, "HPMixedRealityController"));
	EKeys::AddKey(FKeyDetails(HPMixedRealityController_Right_Trigger,
		LOCTEXT("HPMixedRealityController_Right_Trigger_Click", "HP Mixed Reality (R) Trigger"),
		FKeyDetails::GamepadKey | FKeyDetails::NotBlueprintBindableKey, "HPMixedRealityController"));
	EKeys::AddKey(FKeyDetails(HPMixedRealityController_Right_Trigger_Axis,
		LOCTEXT("HPMixedRealityController_Right_Trigger_Axis", "HP Mixed Reality (R) Trigger Axis"),
		FKeyDetails::GamepadKey | FKeyDetails::Axis1D | FKeyDetails::NotBlueprintBindableKey, "HPMixedRealityController"));
	EKeys::AddKey(FKeyDetails(HPMixedRealityController_Right_Thumbstick_X,
		LOCTEXT("HPMixedRealityController_Right_Thumbstick_X", "HP Mixed Reality (R) Thumbstick X"),
		FKeyDetails::GamepadKey | FKeyDetails::Axis1D | FKeyDetails::NotBlueprintBindableKey, "HPMixedRealityController"));
	EKeys::AddKey(FKeyDetails(HPMixedRealityController_Right_Thumbstick_Y,
		LOCTEXT("HPMixedRealityController_Right_Thumbstick_Y", "HP Mixed Reality (R) Thumbstick Y"),
		FKeyDetails::GamepadKey | FKeyDetails::Axis1D | FKeyDetails::NotBlueprintBindableKey, "HPMixedRealityController"));
	EKeys::AddKey(FKeyDetails(HPMixedRealityController_Right_Thumbstick,
		LOCTEXT("HPMixedRealityController_Right_Thumbstick_Click", "HP Mixed Reality (R) Thumbstick"),
		FKeyDetails::GamepadKey | FKeyDetails::NotBlueprintBindableKey, "HPMixedRealityController"));
}

void FHPMotionControllerModule::ShutdownModule()
{
	IModularFeatures::Get().UnregisterModularFeature(GetModularFeatureName(), this);
}

bool FHPMotionControllerModule::GetRequiredExtensions(TArray<const ANSICHAR*>& OutExtensions)
{
	// XR_EXT_HP_MIXED_REALITY_CONTROLLER_EXTENSION_NAME is defined in openxr.h starting with 1.0.10
	OutExtensions.Add(XR_EXT_HP_MIXED_REALITY_CONTROLLER_EXTENSION_NAME);
	return true;
}

void FHPMotionControllerModule::PostCreateInstance(XrInstance InInstance)
{
	XrResult Result = xrStringToPath(InInstance, "/interaction_profiles/hp/mixed_reality_controller", &InteractionProfile);
	check(XR_SUCCEEDED(Result));

	XrPath Path;
	Result = xrStringToPath(InInstance, "/user/hand/left", &Path);
	check(XR_SUCCEEDED(Result));
	ControllerModels.Add(Path, FOpenXRAssetDirectory::HPMixedRealityLeft);

	Result = xrStringToPath(InInstance, "/user/hand/right", &Path);
	check(XR_SUCCEEDED(Result));
	ControllerModels.Add(Path, FOpenXRAssetDirectory::HPMixedRealityRight);
}

bool FHPMotionControllerModule::GetInteractionProfiles(XrInstance InInstance, TArray<FString>& OutKeyPrefixes, TArray<XrPath>& OutPaths, TArray<bool>& OutHasHaptics)
{
	OutKeyPrefixes.Add("HPMixedRealityController");
	OutPaths.Add(InteractionProfile);
	OutHasHaptics.Add(true);

	return true;
}

bool FHPMotionControllerModule::GetControllerModel(XrInstance InInstance, XrPath InInteractionProfile, XrPath InDevicePath, FSoftObjectPath& OutPath)
{
	if (InInteractionProfile == InteractionProfile)
	{
		OutPath = ControllerModels[InDevicePath];
		return true;
	}

	return false;
}

void FHPMotionControllerModule::GetControllerModelsForCooking(TArray<FSoftObjectPath>& OutPaths)
{
	OutPaths.Add(FOpenXRAssetDirectory::HPMixedRealityLeft);
	OutPaths.Add(FOpenXRAssetDirectory::HPMixedRealityRight);
}

#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FHPMotionControllerModule, HPMotionController)
