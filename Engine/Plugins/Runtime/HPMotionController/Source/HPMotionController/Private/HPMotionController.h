// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IOpenXRExtensionPlugin.h"
#include "Modules/ModuleInterface.h"

class FHPMotionControllerModule :
	public IModuleInterface,
	public IOpenXRExtensionPlugin
{
public:
	FHPMotionControllerModule() { }

	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	virtual FString GetDisplayName() override
	{
		return FString(TEXT("HP Motion Controller"));
	}

	bool GetRequiredExtensions(TArray<const ANSICHAR*>& OutExtensions) override;
	void PostCreateInstance(XrInstance InInstance) override;
	bool GetInteractionProfiles(XrInstance InInstance, TArray<FString>& OutKeyPrefixes, TArray<XrPath>& OutPaths, TArray<bool>& OutHasHaptics) override;
	bool GetControllerModel(XrInstance InInstance, XrPath InInteractionProfile, XrPath InDevicePath, FSoftObjectPath& OutPath) override;
	void GetControllerModelsForCooking(TArray<FSoftObjectPath>& OutPaths) override;

private:
	XrPath InteractionProfile;
	TMap<XrPath, FSoftObjectPath> ControllerModels;
};

